{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "0.1",
  "FriendlyName": "NVDEC hardware decoder support for Electra",
  "Description": "Adds codecs from the NVIDIA Media Codec SDK for use with Electra. Which codecs are supported depends on your GPU.\nRequires D3D12.",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "Category": "Media Players",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": true,
  "IsExperimentalVersion": true,
  "Installed": false,
  "Modules": [
    {
      "Name": "NVDECElectra",
      "Type": "Runtime",
      "LoadingPhase": "Default",
      "PlatformAllowList": [ "Win64" ],
      "PlatformArchitectureDenyList": [ "Win64:arm64" ],
      "TargetDenyList": [ "Server" ]
    }
  ],
  "Plugins": [
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    }
  ]
}
