// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class NVDECElectra : ModuleRules
	{
		public NVDECElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
                "RHI",
			    "CUDA",
				"ElectraBase",
                "ElectraSamples",
				"ElectraCodecFactory",
				"ElectraDecoders"
            });
	        PublicDependencyModuleNames.AddRange(new string[] {
			    "nvDecode"
	        });

            bool bHaveNVDEC = false;

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDependencyModuleNames.Add("DirectX");
				PrivateDependencyModuleNames.Add("D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");

                bHaveNVDEC = true;
			    PublicDefinitions.Add("ELECTRA_DECODER_NVDEC_USE_D3D12=1");

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}
			}

            if (bHaveNVDEC)
            {
			    PublicDefinitions.Add("ELECTRA_DECODER_HAVE_NVDEC=1");
            }
            else
            {
			    PublicDefinitions.Add("ELECTRA_DECODER_HAVE_NVDEC=0");
            }
		}
	}
}
