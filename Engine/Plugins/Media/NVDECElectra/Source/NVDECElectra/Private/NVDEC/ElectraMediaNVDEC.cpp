// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaNVDEC.h"
#include "NVDECElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "HAL/PlatformProcess.h"
#include "HAL/IConsoleManager.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H264.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/Google/ElectraBitstreamProcessorVPx.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/AOMedia/ElectraBitstreamProcessor_AV1.h"
#include "Utils/ElectraBitstreamProcessorGenericVideo.h"
#include "Containers/Queue.h"
#include "Stats/Stats.h"
#include "Misc/CoreDelegates.h"
#include "RHI.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "CudaModule.h"

THIRD_PARTY_INCLUDES_START
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <nvcuvid.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
THIRD_PARTY_INCLUDES_END



#ifndef ELECTRA_DECODER_NVDEC_USE_D3D12
#define ELECTRA_DECODER_NVDEC_USE_D3D12 0
#endif
#if ELECTRA_DECODER_NVDEC_USE_D3D12
#define ENABLE_GPU_BUFFER_HELPER 1
#else
#define ENABLE_GPU_BUFFER_HELPER 0
#endif

#define USE_CUDA_STREAM 1

#if ENABLE_GPU_BUFFER_HELPER
#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)
#else
#define ELECTRA_MEDIAGPUBUFFER_DX12 0
#endif

/*********************************************************************************************************************/
#if ELECTRA_DECODER_HAVE_NVDEC
static bool bDisableNVDEC = false;
FAutoConsoleVariableRef CVarElectraDecoderDisableNVDEC(
	TEXT("ElectraDecoders.bDisableNVDEC"),
	bDisableNVDEC,
	TEXT("Disables use of NVDEC"));

DECLARE_CYCLE_STAT(TEXT("ElectraDecoder ConvertOutput"), STAT_ElectraDecoder_ConvertOutputNVDEC, STATGROUP_Media);
#endif
/*********************************************************************************************************************/

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER			2
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				3
#define ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	5

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

#if ELECTRA_DECODER_HAVE_NVDEC

class INVDECMethods
{
public:
	static TSharedPtr<INVDECMethods, ESPMode::ThreadSafe> Get()
	{
		if (!Self)
		{
			Self = MakeShared<INVDECMethods, ESPMode::ThreadSafe>();
		}
		return Self;
	}
	static bool Initialize(void* InDllHandle)
	{
#define LOAD_FROM_DLL(Name) \
	Self->Name = (Name##Ptr)FPlatformProcess::GetDllExport(InDllHandle, TEXT(#Name)); \
	if (Self->Name == nullptr) \
	{ \
		UE_LOG(LogNVDECElectraDecoder, Error, TEXT("Failed to get dll export function: " #Name)); \
		return false; \
	}

#define LOAD_FROM_DLL64(Name) \
	Self->Name = (Name##Ptr)FPlatformProcess::GetDllExport(InDllHandle, TEXT(#Name) "64"); \
	if (Self->Name == nullptr) \
	{ \
		UE_LOG(LogNVDECElectraDecoder, Error, TEXT("Failed to get dll export function: " #Name)); \
		return false; \
	}

		Get();

		LOAD_FROM_DLL(cuvidGetDecoderCaps);
		LOAD_FROM_DLL(cuvidCreateVideoParser);
		LOAD_FROM_DLL(cuvidDestroyVideoParser);
		LOAD_FROM_DLL(cuvidParseVideoData);
		LOAD_FROM_DLL(cuvidCreateDecoder);
		LOAD_FROM_DLL(cuvidDestroyDecoder);
		LOAD_FROM_DLL(cuvidDecodePicture);
		LOAD_FROM_DLL(cuvidGetDecodeStatus);
		LOAD_FROM_DLL(cuvidReconfigureDecoder);
		LOAD_FROM_DLL64(cuvidMapVideoFrame);
		LOAD_FROM_DLL64(cuvidUnmapVideoFrame);
		LOAD_FROM_DLL(cuvidCtxLockCreate);
		LOAD_FROM_DLL(cuvidCtxLockDestroy);
		LOAD_FROM_DLL(cuvidCtxLock);
		LOAD_FROM_DLL(cuvidCtxUnlock);

		Self->DllHandle = InDllHandle;
		return true;
#undef LOAD_FROM_DLL64
#undef LOAD_FROM_DLL
	}
	static void* Destroy()
	{
		void* dh = Self ? Self->DllHandle : nullptr;
		Self.Reset();
		return dh;
	}

	INVDECMethods() = default;
	~INVDECMethods() = default;

	typedef CUresult (*cuvidGetDecoderCapsPtr)(CUVIDDECODECAPS*);

	typedef CUresult (*cuvidCreateVideoParserPtr)(CUvideoparser*, CUVIDPARSERPARAMS*);
	typedef CUresult (*cuvidDestroyVideoParserPtr)(CUvideoparser);
	typedef CUresult (*cuvidParseVideoDataPtr)(CUvideoparser, CUVIDSOURCEDATAPACKET*);

	typedef CUresult (*cuvidCreateDecoderPtr)(CUvideodecoder*, CUVIDDECODECREATEINFO*);
	typedef CUresult (*cuvidDestroyDecoderPtr)(CUvideodecoder);
	typedef CUresult (*cuvidDecodePicturePtr)(CUvideodecoder, CUVIDPICPARAMS*);
	typedef CUresult (*cuvidGetDecodeStatusPtr)(CUvideodecoder, int, CUVIDGETDECODESTATUS*);
	typedef CUresult (*cuvidReconfigureDecoderPtr)(CUvideodecoder, CUVIDRECONFIGUREDECODERINFO*);

	typedef CUresult (*cuvidMapVideoFramePtr)(CUvideodecoder, int, CUdeviceptr*, unsigned int*, CUVIDPROCPARAMS*);
	typedef CUresult (*cuvidUnmapVideoFramePtr)(CUvideodecoder, CUdeviceptr);

	typedef CUresult (*cuvidCtxLockCreatePtr)(CUvideoctxlock*, CUcontext);
	typedef CUresult (*cuvidCtxLockDestroyPtr)(CUvideoctxlock);
	typedef CUresult (*cuvidCtxLockPtr)(CUvideoctxlock, unsigned int);
	typedef CUresult (*cuvidCtxUnlockPtr)(CUvideoctxlock, unsigned int);

	// The DLL handle
	void* DllHandle = nullptr;

	// Decoder capabilities
	cuvidGetDecoderCapsPtr cuvidGetDecoderCaps = nullptr;

	// Parser methods
	cuvidCreateVideoParserPtr cuvidCreateVideoParser = nullptr;
	cuvidDestroyVideoParserPtr cuvidDestroyVideoParser = nullptr;
	cuvidParseVideoDataPtr cuvidParseVideoData = nullptr;

	// Decoder methods
	cuvidCreateDecoderPtr cuvidCreateDecoder = nullptr;
	cuvidDestroyDecoderPtr cuvidDestroyDecoder = nullptr;
	cuvidDecodePicturePtr cuvidDecodePicture = nullptr;
	cuvidGetDecodeStatusPtr cuvidGetDecodeStatus = nullptr;
	cuvidReconfigureDecoderPtr cuvidReconfigureDecoder = nullptr;

	// Frame mapping methods
	cuvidMapVideoFramePtr cuvidMapVideoFrame = nullptr;
	cuvidUnmapVideoFramePtr cuvidUnmapVideoFrame = nullptr;

	// Lock methods
	cuvidCtxLockCreatePtr cuvidCtxLockCreate = nullptr;
	cuvidCtxLockDestroyPtr cuvidCtxLockDestroy = nullptr;
	cuvidCtxLockPtr cuvidCtxLock = nullptr;
	cuvidCtxUnlockPtr cuvidCtxUnlock = nullptr;
	static TSharedPtr<INVDECMethods, ESPMode::ThreadSafe> Self;

	struct FCUDAContextScope
	{
		FCUDAContextScope(CUcontext InContext)
		{
			FCUDAModule::CUDA().cuCtxPushCurrent(InContext);
		}

		~FCUDAContextScope()
		{
			FCUDAModule::CUDA().cuCtxPopCurrent(nullptr);
		}
	};
};
TSharedPtr<INVDECMethods, ESPMode::ThreadSafe> INVDECMethods::Self;



class FNVDECMethods
{
public:
	FNVDECMethods(CUcontext InCudaContext)
		: Methods(INVDECMethods::Get().Get())
		, Context(InCudaContext)
	{ check(Methods); check(Context); }

	CUresult NVDECGetDecoderCaps(CUVIDDECODECAPS* InOutCaps)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidGetDecoderCaps(InOutCaps);
	}

	CUresult NVDECCreateVideoParser(CUvideoparser* OutParser, CUVIDPARSERPARAMS* InParams)
	{
		return Methods->cuvidCreateVideoParser(OutParser, InParams);
	}
	CUresult NVDECDestroyVideoParser(CUvideoparser InParser)
	{
		return Methods->cuvidDestroyVideoParser(InParser);
	}
	CUresult NVDECParseVideoData(CUvideoparser InParser, CUVIDSOURCEDATAPACKET* InPacket)
	{
		return Methods->cuvidParseVideoData(InParser, InPacket);
	}

	CUresult NVDECCreateDecoder(CUvideodecoder* OutDecoder, CUVIDDECODECREATEINFO* InInfo)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidCreateDecoder(OutDecoder, InInfo);
	}
	CUresult NVDECDestroyDecoder(CUvideodecoder InDecoder)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidDestroyDecoder(InDecoder);
	}
	CUresult NVDECDecodePicture(CUvideodecoder InDecoder, CUVIDPICPARAMS* InParams)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidDecodePicture(InDecoder, InParams);
	}
	CUresult NVDECGetDecodeStatus(CUvideodecoder InDecoder, int InPicIdx, CUVIDGETDECODESTATUS* OutStatus)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidGetDecodeStatus(InDecoder, InPicIdx, OutStatus);
	}
	CUresult NVDECReconfigureDecoder(CUvideodecoder InDecoder, CUVIDRECONFIGUREDECODERINFO* InInfo)
	{
		INVDECMethods::FCUDAContextScope cs(Context);
		return Methods->cuvidReconfigureDecoder(InDecoder, InInfo);
	}

	CUresult NVDECMapVideoFrame(CUvideodecoder InDecoder, int InPicIdx, CUdeviceptr* OutDevice, unsigned int* OutPitch, CUVIDPROCPARAMS* InParams)
	{
		return Methods->cuvidMapVideoFrame(InDecoder, InPicIdx, OutDevice, OutPitch, InParams);
	}
	CUresult NVDECUnmapVideoFrame(CUvideodecoder InDecoder, CUdeviceptr InDevice)
	{
		return Methods->cuvidUnmapVideoFrame(InDecoder, InDevice);
	}

	CUresult NVDECCtxLockCreate(CUvideoctxlock* OutLock)
	{
		return Methods->cuvidCtxLockCreate(OutLock, Context);
	}
	CUresult NVDECCtxLockDestroy(CUvideoctxlock InLock)
	{
		return Methods->cuvidCtxLockDestroy(InLock);
	}
	CUresult NVDECCtxLock(CUvideoctxlock InLock, unsigned int InFlags)
	{
		return Methods->cuvidCtxLock(InLock, InFlags);
	}
	CUresult NVDECCtxUnlock(CUvideoctxlock InLock, unsigned int InFlags)
	{
		return Methods->cuvidCtxUnlock(InLock, InFlags);
	}


	class FScopedContext
	{
	public:
		FScopedContext(FNVDECMethods* InThis)
		{
			FCUDAModule::CUDA().cuCtxPushCurrent(InThis->Context);
		}
		void Release()
		{
			if (!bReleased)
			{
				bReleased = true;
				FCUDAModule::CUDA().cuCtxPopCurrent(nullptr);
			}
		}
		~FScopedContext()
		{
			Release();
		}
	private:
		bool bReleased = false;
	};
	friend class FScopedContext;

private:
	INVDECMethods* Methods = nullptr;
	CUcontext Context = nullptr;
};

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderNVDECElectra;


class FVideoDecoderOutputNVDECElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputNVDECElectra()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return DecodedWidth; }
	int32 GetDecodedHeight() const override
	{ return DecodedHeight; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputNVDECElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{ return Codec4CC; }
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBuffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { GPUBuffer.Fence.GetReference(), GPUBuffer.FenceValue };
			return true;
		}
#endif
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorPitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> ColorBuffer;
	EPixelFormat ColorBufferFormat;
	EElectraTextureSamplePixelEncoding ColorBufferEncoding;
	int32 ColorPitch = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
};


class FVideoDecoderNVDECElectra : public FNVDECMethods, public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderNVDECElectra(CUcontext InCudaContext, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderNVDECElectra();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		switch(InitialCodecFormat.FourCC)
		{
			case ElectraDecodersUtil::Make4CC('a','v','c','1'):
			case ElectraDecodersUtil::Make4CC('a','v','c','3'):
			{
				return FElectraDecoderBitstreamProcessorH264::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
			}
			case ElectraDecodersUtil::Make4CC('h','v','c','1'):
			case ElectraDecodersUtil::Make4CC('h','e','v','1'):
			{
				return FElectraDecoderBitstreamProcessorH265::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
			}
			case ElectraDecodersUtil::Make4CC('v','p','0','8'):
			case ElectraDecodersUtil::Make4CC('v','p','0','9'):
			{
				return FElectraDecoderBitstreamProcessorVPx::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
			}
			case ElectraDecodersUtil::Make4CC('a','v','0','1'):
			{
				return FElectraDecoderBitstreamProcessorAV1::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
			}
			default:
			{
				return FElectraDecoderBitstreamProcessorGenericVideo::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
			}
		}
	}

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		bool bDropOutput = false;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();
	bool InternalDecoderDrain(bool bInFlushOnly);

	struct FPendingInput
	{
		CUVIDEOFORMAT Format;
		CUVIDPICPARAMS PicParams;
	};

	struct FOutputInfo
	{
		CUVIDEOFORMAT Format;
		CUVIDPICPARAMS PicParams;
		CUVIDPARSERDISPINFO DispInfo;
		CUVIDDECODECREATEINFO CreateInfo;
	};

	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};

	struct FDecoderStruct
	{
		CUvideoctxlock ContextLock = nullptr;
		CUvideoparser Parser = nullptr;
		CUvideodecoder Decoder = nullptr;
		CUstream CuvidStream = nullptr;
		CUVIDDECODECREATEINFO CreateInfo {};
		CUVIDEOFORMAT CurrentFormat {};
	};

	EConvertResult ConvertDecoderOutput(const FOutputInfo& InOutputInfo);
	bool ConvertDecodedImageToNV12orP010(FVideoDecoderOutputNVDECElectra* InNewOutput, const FOutputInfo& InFormat, CUdeviceptr InDeviceSrcFrame, unsigned int InSrcPitch);

	FDecoderStruct Decoder;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;
	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;
	TMap<int, FPendingInput> PendingInput;
	TArray<TSharedPtr<FVideoDecoderOutputNVDECElectra, ESPMode::ThreadSafe>> PendingOutput;
	IElectraDecoder::FError LastError;
	FString CodecName;
	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bFlushOutput = false;
	uint32 MaxWidth;
	uint32 MaxHeight;
	uint32 MaxOutputBuffers;

#if ELECTRA_MEDIAGPUBUFFER_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif

    static int CUDAAPI HandleVideoSequenceFN(void* InUserData, CUVIDEOFORMAT* InVideoFormat)
	{ return reinterpret_cast<FVideoDecoderNVDECElectra* >(InUserData)->HandleVideoSequence(InVideoFormat); }
    static int CUDAAPI HandlePictureDecodeFN(void* InUserData, CUVIDPICPARAMS* InPicParams)
	{ return reinterpret_cast<FVideoDecoderNVDECElectra* >(InUserData)->HandlePictureDecode(InPicParams); }
    static int CUDAAPI HandlePictureDisplayFN(void* InUserData, CUVIDPARSERDISPINFO* InDispInfo)
	{ return reinterpret_cast<FVideoDecoderNVDECElectra* >(InUserData)->HandlePictureDisplay(InDispInfo); }
    static int CUDAAPI HandleOperatingPointFN(void* InUserData, CUVIDOPERATINGPOINTINFO* InOPInfo)
	{ return reinterpret_cast<FVideoDecoderNVDECElectra* >(InUserData)->HandleOperatingPoint(InOPInfo); }
    static int CUDAAPI HandleSEIMessagesFN(void* InUserData, CUVIDSEIMESSAGEINFO* InSEIMessageInfo)
	{ return reinterpret_cast<FVideoDecoderNVDECElectra* >(InUserData)->HandleSEIMessages(InSEIMessageInfo); }
    int HandleVideoSequence(CUVIDEOFORMAT* InVideoFormat);
    int HandlePictureDecode(CUVIDPICPARAMS* InPicParams);
    int HandlePictureDisplay(CUVIDPARSERDISPINFO* InDispInfo);
    int HandleOperatingPoint(CUVIDOPERATINGPOINTINFO* InOPInfo);
    int HandleSEIMessages(CUVIDSEIMESSAGEINFO* InSEIMessageInfo);
};

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FNVDECElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FNVDECElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FNVDECElectraFactory()
	{ }
	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderNVDECElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (bDisableNVDEC)
		{
			return 0;
		}
		if (!CudaContext)
		{
			return 0;
		}
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vi = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();

		uint32 Width = vi.Width;
		uint32 Height = vi.Height;

		// Check the codecs that NVDEC might support.
		CUVIDDECODECAPS decCaps;
		FMemory::Memzero(decCaps);
		if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','1') || InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','c','3'))
		{
			// We only handle 8 bit here, so at most high profile
			if (vi.Profile.Profile > 100)
			{
				return 0;
			}
			decCaps.eCodecType = cudaVideoCodec_H264;
			decCaps.eChromaFormat = cudaVideoChromaFormat_420;
			decCaps.nBitDepthMinus8 = 0;
		}
		else if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('h','v','c','1') || InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('h','e','v','1'))
		{
			// For best compatibility with slightly older GPUs we only support Main and Main10 profile
			if (vi.Profile.Profile != 1 && vi.Profile.Profile != 2)
			{
				return 0;
			}
			// Level 6.0 is already quite large and at the moment also the highest supported one.
			if (vi.Profile.Level > 180)
			{
				return 0;
			}
			if (vi.BitDepth > 10)
			{
				return 0;
			}
			decCaps.eCodecType = cudaVideoCodec_HEVC;
			decCaps.eChromaFormat = cudaVideoChromaFormat_420;
			decCaps.nBitDepthMinus8 = vi.BitDepth ? vi.BitDepth - 8 : 0;
		}
		else if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('a','v','0','1'))
		{
			// Only profile 0 is supported at the moment
			if (vi.Profile.Profile > 0)
			{
				return 0;
			}
			// Up to level 6.0
			if (vi.Profile.Level > 16)
			{
				return 0;
			}
			decCaps.eCodecType = cudaVideoCodec_AV1;
			decCaps.eChromaFormat = cudaVideoChromaFormat_420;
			decCaps.nBitDepthMinus8 = vi.BitDepth ? vi.BitDepth - 8 : 0;
		}
		else if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','9'))
		{
			// We do need the color info here. If it is not present we return;
			if (!vi.OptColorInfo.IsSet())
			{
				return 0;
			}
			// Only 4:2:0 chroma subsampling
			if (vi.OptColorInfo.GetValue().chromaSubsampling != 0 && vi.OptColorInfo.GetValue().chromaSubsampling != 1)
			{
				return 0;
			}
			// Only profile 0 and 2
			if (vi.Profile.Profile != 0 && vi.Profile.Profile != 2)
			{
				return 0;
			}
			decCaps.eCodecType = cudaVideoCodec_VP9;
			decCaps.eChromaFormat = cudaVideoChromaFormat_420;
			decCaps.nBitDepthMinus8 = vi.BitDepth ? vi.BitDepth - 8 : 0;
		}
		else if (InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','8'))
		{
			decCaps.eCodecType = cudaVideoCodec_VP8;
			decCaps.eChromaFormat = cudaVideoChromaFormat_420;
			decCaps.nBitDepthMinus8 = vi.BitDepth ? vi.BitDepth - 8 : 0;
		}
		else
		{
			return 0;
		}

		// See if we already checked for this format to avoid the call into nvdec.
		CUresult Result = CUDA_SUCCESS;
		SupportedFormatsLock.Lock();
		int32 FmtIdx = SupportedFormats.IndexOfByPredicate([&decCaps](const CUVIDDECODECAPS& InCaps) { return decCaps.eCodecType == InCaps.eCodecType && decCaps.eChromaFormat == InCaps.eChromaFormat && decCaps.nBitDepthMinus8 == InCaps.nBitDepthMinus8;});
		if (FmtIdx != INDEX_NONE)
		{
			decCaps = SupportedFormats[FmtIdx];
		}
		else
		{
			INVDECMethods::FCUDAContextScope cs(CudaContext);
			Result = INVDECMethods::Get()->cuvidGetDecoderCaps(&decCaps);
			if (Result == CUDA_SUCCESS)
			{
				SupportedFormats.Emplace(decCaps);
			}
		}
		SupportedFormatsLock.Unlock();

		if (Result == CUDA_SUCCESS)
		{
			if (decCaps.bIsSupported)
			{
				if ((Width && (Width < decCaps.nMinWidth || Width > decCaps.nMaxWidth)) ||
					(Height && (Height < decCaps.nMinHeight || Height > decCaps.nMaxHeight)) ||
					((Width >> 4) * (Height >> 4) > decCaps.nMaxMBCount))
				{
					UE_LOGF(LogNVDECElectraDecoder, VeryVerbose, "NVDEC does not support decoding of \"%ls\" at %u*%u on your GPU", *InCodecFormat.RFC6381, Width, Height);
					return 0;
				}
				// Vendor GPU accelerated decoding to return a large confidence score that should cause this to be used.
				return 100;
			}
			else
			{
				UE_LOGF(LogNVDECElectraDecoder, VeryVerbose, "NVDEC does not support decoding of \"%ls\" on your GPU", *InCodecFormat.RFC6381);
			}
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		if (bDisableNVDEC)
		{
			return nullptr;
		}
		UE_LOGF(LogNVDECElectraDecoder, Verbose, "Created an NVDEC decoder.");
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderNVDECElectra, ESPMode::ThreadSafe>(CudaContext, InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("NVDEC")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString(TEXT("Nvidia"))); }


	void SetCUDAContext(CUcontext InContext)
	{
		CudaContext = InContext;
	}

	mutable FCriticalSection SupportedFormatsLock;
	mutable TArray<CUVIDDECODECAPS> SupportedFormats;

	CUcontext CudaContext = nullptr;
	static TSharedPtr<FNVDECElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FNVDECElectraFactory, ESPMode::ThreadSafe> FNVDECElectraFactory::Self;

#endif // ELECTRA_DECODER_HAVE_NVDEC

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaNVDECDecoder::Startup()
{
#if ELECTRA_DECODER_HAVE_NVDEC
	// Check minimum driver version.
	if (!IsRHIDeviceNVIDIA())
	{
		UE_LOGF(LogNVDECElectraDecoder, Log, "RHI device is not from NVIDIA, cannot use NVDEC acceleration.");
		return;
	}
	const TCHAR* const MinimumDriverVersion = TEXT("531.61");
	const TCHAR* const DllName = TEXT("nvcuvid.dll");

	if (FDriverVersion(GRHIAdapterUserDriverVersion) < FDriverVersion(FString(MinimumDriverVersion)))
	{
		UE_LOGF(LogNVDECElectraDecoder, Log, "Driver version %ls is older than minimum required %ls. Cannot use NVDEC acceleration, please update your drivers.", *GRHIAdapterUserDriverVersion, MinimumDriverVersion);
		return;
	}
	void* DllHandle = FPlatformProcess::GetDllHandle(DllName);
	if (!DllHandle)
	{
		UE_LOGF(LogNVDECElectraDecoder, Log, "Failed to get NVDEC DLL handle from %ls. Cannot use NVDEC acceleration", DllName);
		return;
	}
	if (!INVDECMethods::Initialize(DllHandle))
	{
		UE_LOGF(LogNVDECElectraDecoder, Warning, "Failed to get all required NVDEC method pointers. Cannot use NVDEC acceleration");
		FPlatformProcess::FreeDllHandle(DllHandle);
		return;
	}

	// Load the necessary modules if they are not yet.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));
	FCUDAModule& CUDAModule = FModuleManager::LoadModuleChecked<FCUDAModule>(TEXT("CUDA"));
	if (CUDAModule.IsAvailable())
	{
		// It is important that our plugin here is loaded sooner than the `PostEngineInit` phase due an issue with the CUDA module
		// us not notifying via the `OnPostCUDAInit` delegate *IF* it has already been initialized. The order of initialization is
		// not guaranteed at that stage, so we need to come here "earlier" so our delegate will get called.
		CUDAModule.OnPostCUDAInit.AddLambda([]()
		{
			FCUDAModule& CUDAModule = FModuleManager::GetModuleChecked<FCUDAModule>(TEXT("CUDA"));
			FNVDECElectraFactory::Self = MakeShared<FNVDECElectraFactory, ESPMode::ThreadSafe>();
			FNVDECElectraFactory::Self->SetCUDAContext(CUDAModule.GetCudaContext());
			// Register as modular feature.
			IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FNVDECElectraFactory::Self.Get());
		});
	}
	else
	{
		UE_LOGF(LogNVDECElectraDecoder, Log, "CUDA is not available, cannot use NVDEC acceleration");
	}
#endif
}

void FElectraMediaNVDECDecoder::Shutdown()
{
#if ELECTRA_DECODER_HAVE_NVDEC
	if (FNVDECElectraFactory::Self.Get())
	{
		IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FNVDECElectraFactory::Self.Get());
		FNVDECElectraFactory::Self.Reset();
	}
	void* DllHandle = INVDECMethods::Destroy();
	if (DllHandle)
	{
		FPlatformProcess::FreeDllHandle(DllHandle);
	}
#endif
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/
#if ELECTRA_DECODER_HAVE_NVDEC

FVideoDecoderNVDECElectra::FVideoDecoderNVDECElectra(CUcontext InCudaContext, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: FNVDECMethods(InCudaContext)
	, InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	//const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	switch(InitialCodecFormat.FourCC)
	{
		case ElectraDecodersUtil::Make4CC('a','v','c','1'):
		case ElectraDecodersUtil::Make4CC('a','v','c','3'):
		{
			CodecName = TEXT("AVC");
			break;
		}
		case ElectraDecodersUtil::Make4CC('h','e','v','1'):
		case ElectraDecodersUtil::Make4CC('h','v','c','1'):
		{
			CodecName = TEXT("HEVC");
			break;
		}
		case ElectraDecodersUtil::Make4CC('v','p','0','8'):
		{
			CodecName = TEXT("VP8");
			break;
		}
		case ElectraDecodersUtil::Make4CC('v','p','0','9'):
		{
			CodecName = TEXT("VP9");
			break;
		}
		case ElectraDecodersUtil::Make4CC('a','v','0','1'):
		{
			CodecName = TEXT("AV1");
			break;
		}
		default:
		{
			CodecName = TEXT("unknown");
			break;
		}
	}
	MaxWidth = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_width"), 1920), 16);
	MaxHeight = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_height"), 1080), 16);
	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
}


FVideoDecoderNVDECElectra::~FVideoDecoderNVDECElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderNVDECElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderNVDECElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderNVDECElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderNVDECElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderNVDECElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	if (!Decoder.Parser)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderNVDECElectra::ResetToCleanStart()
{
	InternalDecoderDestroy();
	InDecoderInput.Empty();
	PendingInput.Empty();
	PendingOutput.Empty();
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

IElectraDecoder::EDecoderError FVideoDecoderNVDECElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	if (D3D12ResourcePool.IsValid() && !D3D12ResourcePool->BufferAvailable())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
#endif

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// If we have enough pending output (internally only, excluding however many are in flight in the media framework queue)
	// we wait decoding additional ones.
	if (PendingOutput.Num() >= (int32)MaxOutputBuffers-1)
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Create parser if necessary (the decoder is created later)
	if (!Decoder.Parser && !InternalDecoderCreate(InInputAccessUnit, InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode data.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
		In->AdditionalOptions = InAdditionalOptions;
		In->bDropOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::DoNotOutput;
		In->AccessUnit = InInputAccessUnit;
		In->AccessUnit.Data = nullptr;
		In->AccessUnit.DataSize = 0;

		void* AUData = nullptr;
		uint32 AUSize = InInputAccessUnit.DataSize;
		// If this is a sync sample we prepend the CSD. While this is not necessary on a running stream we need to have the CSD
		// on the first frame and it is easier to prepend it to all IDR frames when seeking etc.
		if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
		{
			TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
			if (CSD.Num())
			{
				AUSize += CSD.Num();
				AUData = FMemory::Malloc(AUSize);
				if (!AUData)
				{
					PostError(-1, TEXT("Out of memory, failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}
				FMemory::Memcpy(AUData, CSD.GetData(), CSD.Num());
				FMemory::Memcpy(ElectraDecodersUtil::AdvancePointer(AUData, CSD.Num()), InInputAccessUnit.Data, InInputAccessUnit.DataSize);
			}
		}

		InDecoderInput.Emplace(MoveTemp(In));
		InDecoderInput.StableSort([](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& a, const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& b)
		{
			return a->AccessUnit.PTS < b->AccessUnit.PTS;
		});
		CUVIDSOURCEDATAPACKET dp {};
		// Note that we cannot utilize `CUVID_PKT_DISCONTINUITY` as this will interpolate the frame PTS
		// rendering it useless for our use to correlate output and input!
		dp.flags = CUvideopacketflags::CUVID_PKT_TIMESTAMP | CUvideopacketflags::CUVID_PKT_ENDOFPICTURE;
		dp.payload_size = AUSize;
		dp.payload = reinterpret_cast<const unsigned char *>(AUData ? AUData : InInputAccessUnit.Data);
		dp.timestamp = InInputAccessUnit.PTS.GetTicks();
		CUresult Result = NVDECParseVideoData(Decoder.Parser, &dp);
		if (AUData)
		{
			FMemory::Free(AUData);
			AUData = nullptr;
		}
		if (LastError.IsSet())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (Result == CUDA_SUCCESS)
		{
			return IElectraDecoder::EDecoderError::None;
		}
		else
		{
			PostError(Result, TEXT("Failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FVideoDecoderNVDECElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (Decoder.Parser)
	{
		DecodeState = EDecodeState::Draining;
		if (!InternalDecoderDrain(false))
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderNVDECElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	InternalDecoderDrain(true);
	InDecoderInput.Empty();
	PendingInput.Empty();
	PendingOutput.Empty();
	DecodeState = EDecodeState::Decoding;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderNVDECElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	// Have output?
	if (!PendingOutput.IsEmpty())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	// When draining we immediately get all pending output delivered, so when
	// we get here (PendingOutput being empty) while draining we know that
	// draining is complete.
	if (DecodeState == EDecodeState::Draining)
	{
		InDecoderInput.Empty();
		PendingInput.Empty();
		DecodeState = EDecodeState::Decoding;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}

	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderNVDECElectra::GetOutput()
{
	check(!PendingOutput.IsEmpty());
	TSharedPtr<FVideoDecoderOutputNVDECElectra, ESPMode::ThreadSafe> Out;
	if (PendingOutput.Num())
	{
		Out = PendingOutput[0];
		PendingOutput.RemoveAt(0);
	}
	return Out;
}

bool FVideoDecoderNVDECElectra::InternalDecoderCreate(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	CUVIDPARSERPARAMS pp {};
	switch(InitialCodecFormat.FourCC)
	{
		case ElectraDecodersUtil::Make4CC('a','v','c','1'):
		case ElectraDecodersUtil::Make4CC('a','v','c','3'):
		{
			pp.CodecType = cudaVideoCodec_H264;
			break;
		}
		case ElectraDecodersUtil::Make4CC('h','v','c','1'):
		case ElectraDecodersUtil::Make4CC('h','e','v','1'):
		{
			pp.CodecType = cudaVideoCodec_HEVC;
			break;
		}
		case ElectraDecodersUtil::Make4CC('v','p','0','8'):
		{
			pp.CodecType = cudaVideoCodec_VP8;
			break;
		}
		case ElectraDecodersUtil::Make4CC('v','p','0','9'):
		{
			pp.CodecType = cudaVideoCodec_VP9;
			break;
		}
		case ElectraDecodersUtil::Make4CC('a','v','0','1'):
		{
			pp.CodecType = cudaVideoCodec_AV1;
			pp.bAnnexb = 0;
			pp.pfnGetOperatingPoint = HandleOperatingPointFN;
			break;
		}
		default:
		{
			return PostError(0, TEXT("Failed to create decoder, unhandled 4CC"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		}
	}

	CUresult Result;
	// Create a lock that is needed for some nvdec functions.
	Result = NVDECCtxLockCreate(&Decoder.ContextLock);
	if (Result != CUDA_SUCCESS)
	{
		return PostError(Result, TEXT("Failed to create decoder, could not create lock"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

#if USE_CUDA_STREAM
	FScopedContext sc(this);
	Result = FCUDAModule::CUDA().cuStreamCreate(&Decoder.CuvidStream, CU_STREAM_DEFAULT);
	if (Result != CUDA_SUCCESS)
	{
		return PostError(Result, TEXT("Failed to create decoder, could not create stream"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	sc.Release();
#endif


	// Create a bitstream parser. This does not require the lock or the context.
	pp.ulMaxNumDecodeSurfaces = 1;
	pp.ulMaxDisplayDelay = 1;
	pp.pUserData = this;
	pp.pfnSequenceCallback = HandleVideoSequenceFN;
	pp.pfnDecodePicture = HandlePictureDecodeFN;
	pp.pfnDisplayPicture = HandlePictureDisplayFN;
	Result = NVDECCreateVideoParser(&Decoder.Parser, &pp);
	if (Result != CUDA_SUCCESS)
	{
		return PostError(Result, TEXT("Failed to create decoder, could not create parser"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	// We only create the parser here. The decoder gets created on demand during the parser callbacks.
	return true;
}

void FVideoDecoderNVDECElectra::InternalDecoderDestroy()
{
	if (Decoder.Decoder)
	{
		NVDECDestroyDecoder(Decoder.Decoder);
		Decoder.Decoder = nullptr;
	}
	if (Decoder.Parser)
	{
		NVDECDestroyVideoParser(Decoder.Parser);
		Decoder.Parser = nullptr;
	}
	if (Decoder.CuvidStream)
	{
		FScopedContext sc(this);
		FCUDAModule::CUDA().cuStreamDestroy(Decoder.CuvidStream);
		Decoder.CuvidStream = nullptr;
	}
	if (Decoder.ContextLock)
	{
		NVDECCtxLockDestroy(Decoder.ContextLock);
		Decoder.ContextLock = nullptr;
	}
	FMemory::Memzero(Decoder.CreateInfo);
	FMemory::Memzero(Decoder.CurrentFormat);
}

bool FVideoDecoderNVDECElectra::InternalDecoderDrain(bool bInFlushOnly)
{
	if (Decoder.Parser)
	{
		CUVIDSOURCEDATAPACKET dp {};
		dp.flags = CUvideopacketflags::CUVID_PKT_ENDOFSTREAM | CUvideopacketflags::CUVID_PKT_NOTIFY_EOS;
		bFlushOutput = bInFlushOnly;
		CUresult Result = NVDECParseVideoData(Decoder.Parser, &dp);
		bFlushOutput = false;
		if (Result != CUDA_SUCCESS)
		{
			return PostError(Result, TEXT("Failed to send EOS packet"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
		}
	}
	return true;
}


int FVideoDecoderNVDECElectra::HandleVideoSequence(CUVIDEOFORMAT* InVideoFormat)
{
	// Check again if the format is supported. The initial check may not have been
	// accurate since it relied on externally provided stream information (from container or playlist).
	CUVIDDECODECAPS DecodeCaps {};
	DecodeCaps.eCodecType = InVideoFormat->codec;
	DecodeCaps.eChromaFormat = InVideoFormat->chroma_format;
	DecodeCaps.nBitDepthMinus8 = InVideoFormat->bit_depth_luma_minus8;
	CUresult Result = NVDECGetDecoderCaps(&DecodeCaps);
	if (Result != CUDA_SUCCESS || !DecodeCaps.bIsSupported)
	{
		PostError(Result, TEXT("Codec not supported on this GPU"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}
	if ((InVideoFormat->coded_width < DecodeCaps.nMinWidth) || (InVideoFormat->coded_height < DecodeCaps.nMinHeight))
	{
		PostError(Result, FString::Printf(TEXT("Resolution of %ux%u is less than minimum resolution of %ux%u required by this GPU"), InVideoFormat->coded_width, InVideoFormat->coded_height, DecodeCaps.nMinWidth, DecodeCaps.nMinHeight), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}
	if ((InVideoFormat->coded_width > DecodeCaps.nMaxWidth) || (InVideoFormat->coded_height > DecodeCaps.nMaxHeight))
	{
		PostError(Result, FString::Printf(TEXT("Resolution of %ux%u exceeds max resolution of %ux%u supported by this GPU"), InVideoFormat->coded_width, InVideoFormat->coded_height, DecodeCaps.nMaxWidth, DecodeCaps.nMaxHeight), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}
	if ((InVideoFormat->coded_width >> 4) * (InVideoFormat->coded_height >> 4) > DecodeCaps.nMaxMBCount)
	{
		PostError(Result, FString::Printf(TEXT("Exceeding max macroblock count of %u supported by this GPU"), DecodeCaps.nMaxMBCount), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}
	if (InVideoFormat->bit_depth_luma_minus8 == 0 && (DecodeCaps.nOutputFormatMask & (1 << cudaVideoSurfaceFormat_NV12)) == 0)
	{
		PostError(Result, FString::Printf(TEXT("NV12 output format for 8 bit content is not supported by this GPU")), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}
	if (InVideoFormat->bit_depth_luma_minus8 > 0 && (DecodeCaps.nOutputFormatMask & (1 << cudaVideoSurfaceFormat_P016)) == 0)
	{
		PostError(Result, FString::Printf(TEXT("P016 output format for 10/12 bit content is not supported by this GPU")), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return 0;
	}

	// If we already have a decoder see if it requires reconfiguration
	if (Decoder.Decoder)
	{
		/*
			cannot actually check for this since the coded size may be larger than the max size which is the cropped size (1080 instead of the coded 1088 for example).

			if (InVideoFormat->coded_width > MaxWidth || InVideoFormat->coded_height > MaxHeight)
			{
				PostError(0, FString::Printf(TEXT("Mid-stream resolution change exceeds maximum configured dimensions!")), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
				return 0;
			}
		*/

		// Changes in bit depth or chroma format require a new decoder.
		if (Decoder.CurrentFormat.bit_depth_luma_minus8 != InVideoFormat->bit_depth_luma_minus8 ||
			Decoder.CurrentFormat.bit_depth_chroma_minus8 != InVideoFormat->bit_depth_chroma_minus8 ||
			Decoder.CurrentFormat.chroma_format != InVideoFormat->chroma_format)
		{
			// Need a new decoder. Destroy the one we have.
			Result = NVDECDestroyDecoder(Decoder.Decoder);
			Decoder.Decoder = nullptr;
		}
		else
		{
			bool bResolutionsChanged = Decoder.CurrentFormat.coded_width != InVideoFormat->coded_width ||  Decoder.CurrentFormat.coded_height != InVideoFormat->coded_height;
			/*
			bool bDisplayRectChanged = Decoder.CurrentFormat.display_area.bottom != InVideoFormat->display_area.bottom || Decoder.CurrentFormat.display_area.top != InVideoFormat->display_area.top ||
									   Decoder.CurrentFormat.display_area.left != InVideoFormat->display_area.left || Decoder.CurrentFormat.display_area.right != InVideoFormat->display_area.right;
			*/
			bool bDPBSizeChanged = Decoder.CurrentFormat.min_num_decode_surfaces < InVideoFormat->min_num_decode_surfaces;
			bool bReconfNeeded = bResolutionsChanged || bDPBSizeChanged;
			if (bReconfNeeded)
			{
			    CUVIDRECONFIGUREDECODERINFO rp {};
				rp.ulWidth = InVideoFormat->coded_width;
				rp.ulHeight = InVideoFormat->coded_height;
				rp.ulTargetWidth = InVideoFormat->coded_width;
				rp.ulTargetHeight = InVideoFormat->coded_height;
				rp.ulNumDecodeSurfaces = bDPBSizeChanged ? InVideoFormat->min_num_decode_surfaces : Decoder.CurrentFormat.min_num_decode_surfaces;

				Result = NVDECReconfigureDecoder(Decoder.Decoder, &rp);
				if (Result == CUDA_SUCCESS)
				{
					Decoder.CurrentFormat = *InVideoFormat;
					return InVideoFormat->min_num_decode_surfaces;
				}
				else
				{
					PostError(Result, FString::Printf(TEXT("Failed to reconfigure decoder on mid-stream resolution change!")), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return 0;
				}
			}
			else
			{
				// Reconfiguration not needed. Still remember the now current format and just return.
				Decoder.CurrentFormat = *InVideoFormat;
				return 1;
			}
		}
	}
	Decoder.CurrentFormat = *InVideoFormat;

	// Do we have to create a new decoder or reconfigure the existing one?
	if (Decoder.Decoder == nullptr)
	{
		CUVIDDECODECREATEINFO& ci(Decoder.CreateInfo);
		FMemory::Memzero(ci);
		ci.ulWidth = InVideoFormat->coded_width;
		ci.ulHeight = InVideoFormat->coded_height;
		ci.ulNumDecodeSurfaces = InVideoFormat->min_num_decode_surfaces;
		ci.CodecType = InVideoFormat->codec;
		ci.ChromaFormat = InVideoFormat->chroma_format;
		ci.ulCreationFlags = cudaVideoCreateFlags::cudaVideoCreate_PreferCUVID;
		ci.bitDepthMinus8 = InVideoFormat->bit_depth_luma_minus8;
		ci.ulMaxWidth = MaxWidth;
		ci.ulMaxHeight = MaxHeight;
		ci.OutputFormat = InVideoFormat->bit_depth_luma_minus8 ? cudaVideoSurfaceFormat_P016 : cudaVideoSurfaceFormat_NV12;
		ci.DeinterlaceMode = cudaVideoDeinterlaceMode::cudaVideoDeinterlaceMode_Weave;
		ci.ulTargetWidth = InVideoFormat->coded_width;
		ci.ulTargetHeight = InVideoFormat->coded_height;
		ci.ulNumOutputSurfaces = 1;		// 1 surface is sufficient as we lock it and copy out its data right away without delay.
		ci.vidLock = Decoder.ContextLock;
		Result = NVDECCreateDecoder(&Decoder.Decoder, &ci);
		if (Result == CUDA_SUCCESS)
		{
			return InVideoFormat->min_num_decode_surfaces;
		}
		else
		{
			PostError(Result, FString::Printf(TEXT("Failed to create decoder")), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
			return 0;
		}
	}
	return 0;
}

int FVideoDecoderNVDECElectra::HandlePictureDecode(CUVIDPICPARAMS* InPicParams)
{
	check(DecodeState != EDecodeState::Draining);
	// Nothing special to do here, just call the decode method.
	CUresult Result = NVDECDecodePicture(Decoder.Decoder, InPicParams);
	if (Result == CUDA_SUCCESS)
	{
		// Add this to the list of pending inputs.
		FPendingInput pi;
		pi.Format = Decoder.CurrentFormat;
		pi.PicParams = *InPicParams;
		PendingInput.Add(InPicParams->CurrPicIdx, MoveTemp(pi));
		return 1;
	}
	PostError(Result, FString::Printf(TEXT("Failed to decode frame")), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
	return 0;
}

int FVideoDecoderNVDECElectra::HandlePictureDisplay(CUVIDPARSERDISPINFO* InDispInfo)
{
	// Is this the notification that on draining we have received all frames?
	if (InDispInfo == nullptr)
	{
		return 1;
	}
	else if (bFlushOutput)
	{
		PendingInput.Remove(InDispInfo->picture_index);
		return 1;
	}
	else
	{
		FPendingInput pi;
		if (!PendingInput.RemoveAndCopyValue(InDispInfo->picture_index, pi))
		{
			PostError(0, FString::Printf(TEXT("Picture for display not found in pending input")), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return 0;
		}
		FOutputInfo po;
		po.Format = pi.Format;
		po.PicParams = pi.PicParams;
		po.DispInfo = *InDispInfo;
		po.CreateInfo = Decoder.CreateInfo;
		EConvertResult Result = ConvertDecoderOutput(po);
		return Result == EConvertResult::Failure ? 0 : 1;
	}
}

int FVideoDecoderNVDECElectra::HandleOperatingPoint(CUVIDOPERATINGPOINTINFO* InOPInfo)
{
	return 0;
}

int FVideoDecoderNVDECElectra::HandleSEIMessages(CUVIDSEIMESSAGEINFO* InSEIMessageInfo)
{
	return 0;
}


FVideoDecoderNVDECElectra::EConvertResult FVideoDecoderNVDECElectra::ConvertDecoderOutput(const FOutputInfo& InOutputInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraDecoder_ConvertOutputNVDEC);

	// Get the optional decode status. We don't do anything with it at the moment though.
	// NVDECGetDecodeStatus() may return CUDA_ERROR_NOT_SUPPORTED for some codecs or GPUs!
	CUVIDGETDECODESTATUS DecodeStatus { };
	CUresult Result = NVDECGetDecodeStatus(Decoder.Decoder, InOutputInfo.DispInfo.picture_index, &DecodeStatus);
	if (Result == CUDA_SUCCESS && DecodeStatus.decodeStatus == cuvidDecodeStatus_Invalid)
	{
		PostError(Result, TEXT("Got invalid decode status!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	// Find the input corresponding to this output.
	FTimespan FramePTS((int64)InOutputInfo.DispInfo.timestamp);
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec]->AccessUnit.PTS == FramePTS)
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		// It is possible that with VP9 we get more than one output corresponding to the same
		// input access unit (and thus PTS) if the AU contains a superframe.
		if (InitialCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','9'))
		{
			// We cannot return more than one frame to the upper level and thus need to ignore this frame.
			return EConvertResult::Success;
		}
		else
		{
			PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return EConvertResult::Failure;
		}
	}
	// Remove entries that are severely outdated and have not been returned for some reason.
	if (DecodeState != EDecodeState::Draining)
	{
		FramePTS -= FTimespan::FromSeconds(1.0);
		InDecoderInput.RemoveAll([&](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e)
		{
			return e->AccessUnit.PTS < FramePTS;
		});
	}

	// Output to be dropped?
	if (In->bDropOutput)
	{
		return EConvertResult::Success;
	}

	TSharedPtr<FVideoDecoderOutputNVDECElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputNVDECElectra>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;
	NewOutput->Width = InOutputInfo.Format.display_area.right - InOutputInfo.Format.display_area.left;
	NewOutput->Height = InOutputInfo.Format.display_area.bottom - InOutputInfo.Format.display_area.top;
	NewOutput->NumBits = InOutputInfo.Format.bit_depth_luma_minus8 + 8;
	NewOutput->AspectW = InOutputInfo.Format.display_aspect_ratio.x;
	NewOutput->AspectH = InOutputInfo.Format.display_aspect_ratio.y;
	NewOutput->Codec4CC = InitialCodecFormat.FourCC;

	// Map the frame.
	CUVIDPROCPARAMS videoProcessingParameters {};
	videoProcessingParameters.progressive_frame = InOutputInfo.DispInfo.progressive_frame;
	videoProcessingParameters.second_field = InOutputInfo.DispInfo.repeat_first_field + 1;
	videoProcessingParameters.top_field_first = InOutputInfo.DispInfo.top_field_first;
	videoProcessingParameters.unpaired_field = InOutputInfo.DispInfo.repeat_first_field < 0;
	videoProcessingParameters.output_stream = Decoder.CuvidStream;

	FScopedContext sc(this);

	CUdeviceptr dpSrcFrame = 0;
	unsigned int nSrcPitch = 0;
	Result = NVDECMapVideoFrame(Decoder.Decoder, InOutputInfo.DispInfo.picture_index, &dpSrcFrame, &nSrcPitch, &videoProcessingParameters);
	if (Result != CUDA_SUCCESS)
	{
		PostError(Result, TEXT("Failed to map video frame (cuvidMapVideoFrame)!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}
	bool bConvOk = ConvertDecodedImageToNV12orP010(NewOutput.Get(), InOutputInfo, dpSrcFrame, nSrcPitch);
	/*Result =*/ NVDECUnmapVideoFrame(Decoder.Decoder, dpSrcFrame);
	sc.Release();
	if (!bConvOk)
	{
		// Error should have been posted already.
		return EConvertResult::Failure;
	}

	NewOutput->ExtraValues.Emplace(TEXT("pixfmt"), FVariant((int64)(NewOutput->NumBits == 8 ? EPixelFormat::PF_NV12 : EPixelFormat::PF_P010)));
	NewOutput->ExtraValues.Emplace(TEXT("pixenc"), FVariant((int64)EElectraTextureSamplePixelEncoding::Native));
	/*
		Scale factor is not needed because the output format is P016 for 10 or 12 bits.

		if (NewOutput->NumBits == 10)
		{
			NewOutput->ExtraValues.Emplace(TEXT("pix_datascale"), FVariant(64.0));
		}
		else if (NewOutput->NumBits == 12)
		{
			NewOutput->ExtraValues.Emplace(TEXT("pix_datascale"), FVariant(16.0));
		}
	*/

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("nvdec")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(CodecName));
	PendingOutput.Emplace(MoveTemp(NewOutput));
	return EConvertResult::Success;
}

bool FVideoDecoderNVDECElectra::ConvertDecodedImageToNV12orP010(FVideoDecoderOutputNVDECElectra* InNewOutput, const FOutputInfo& InFormat, CUdeviceptr InDeviceSrcFrame, unsigned int InSrcPitch)
{
	// Double check that the decoder output format is the one we're expecting.
	if (InFormat.CreateInfo.OutputFormat != cudaVideoSurfaceFormat_NV12 && InFormat.CreateInfo.OutputFormat != cudaVideoSurfaceFormat_P016)
	{
		PostError(0, FString::Printf(TEXT("Unsupported decoded image format (%d)"), InFormat.CreateInfo.OutputFormat), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	void* PlatformDevice = nullptr;
	int32 PlatformDeviceVersion = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
#endif
	bool bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);

	int32 BytesPerPixel = 1;
	if (InFormat.CreateInfo.OutputFormat == cudaVideoSurfaceFormat_NV12)
	{
		InNewOutput->NumBuffers = 1;
		InNewOutput->ColorBufferFormat = EPixelFormat::PF_NV12;
		InNewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		InNewOutput->DecodedWidth = InNewOutput->Width;
		InNewOutput->DecodedHeight = bUseGPUBuffers ? InNewOutput->Height : (InNewOutput->Height * 3 / 2);
	}
	else if (InFormat.CreateInfo.OutputFormat == cudaVideoSurfaceFormat_P016)
	{
		InNewOutput->NumBuffers = 1;
		InNewOutput->ColorBufferFormat = EPixelFormat::PF_P010;
		InNewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		InNewOutput->DecodedWidth = InNewOutput->Width;
		InNewOutput->DecodedHeight = bUseGPUBuffers ? InNewOutput->Height : (InNewOutput->Height * 3 / 2);
		BytesPerPixel = 2;
	}

	const int32 w = InNewOutput->Width;
	const int32 h = InNewOutput->Height;
	const int32 aw = Align(w, 2);
	const int32 ah = Align(h, 2);

	uint8* DstY = nullptr;
	uint8* DstUV = nullptr;
	uint32 DstPitch = 0;

	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe>& OutCPUBuffer = InNewOutput->ColorBuffer;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (!bUseGPUBuffers)
#endif
	{
		uint32 Pitch = aw * BytesPerPixel;
		int32 AllocSize = Pitch * (ah * 3 / 2);

#if ELECTRA_MEDIAGPUBUFFER_DX12
		InNewOutput->GPUBuffer.Resource = nullptr;
		InNewOutput->GPUBuffer.Fence = nullptr;
#endif

		OutCPUBuffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
		OutCPUBuffer->AddUninitialized(AllocSize);
		DstY = OutCPUBuffer->GetData();
		DstUV = DstY + Pitch * ah;
		DstPitch = Pitch;
	}
#if ELECTRA_MEDIAGPUBUFFER_DX12
	else
	{
		TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
		HRESULT Result;
		FString ResultMsg;

		OutCPUBuffer.Reset();
		InNewOutput->GPUBuffer.Resource = nullptr;

		// Create the resource pool as needed...
		if (!D3D12ResourcePool)
		{
			D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers + kElectraDecoderPipelineExtraFrames, MaxWidth, MaxHeight * 3 / 2, BytesPerPixel);
		}

		// Request resource and fence...
		uint32 BufferPitch;
		if (D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, InNewOutput->GPUBuffer, BufferPitch))
		{
			// Correct pitch to reflect resource's setup
			InNewOutput->ColorPitch = BufferPitch;

			// Map the buffer so we can get a CPU address to the WC configured buffer
			Result = InNewOutput->GPUBuffer.Resource->Map(0, nullptr, (void**)&DstY);
			if (FAILED(Result))
			{
				return false;
			}
			DstUV = DstY + BufferPitch * ah;
			DstPitch = BufferPitch;
		}
		else
		{
			return false;
		}
	}
#endif // ELECTRA_MEDIAGPUBUFFER_DX12

	InNewOutput->ColorPitch = DstPitch;

	// Copy luma plane
	CUDA_MEMCPY2D m { };
	m.srcMemoryType = CU_MEMORYTYPE_DEVICE;
	m.srcDevice = InDeviceSrcFrame;
	m.srcPitch = InSrcPitch;
	m.dstMemoryType = CU_MEMORYTYPE_HOST;
	m.dstHost = DstY;
	m.dstPitch = DstPitch;
	m.WidthInBytes = aw * BytesPerPixel;
	m.Height = ah;

	CUresult Result;
	Result = Decoder.CuvidStream ? FCUDAModule::CUDA().cuMemcpy2DAsync(&m, Decoder.CuvidStream) : FCUDAModule::CUDA().cuMemcpy2D(&m);
	check(Result == CUDA_SUCCESS);

	// Copy chroma plane
	// NVDEC output has luma height aligned by 2. Adjust chroma offset by aligning height
	m.srcDevice = (CUdeviceptr)((uint8_t *)InDeviceSrcFrame + m.srcPitch * InFormat.Format.coded_height);
	m.dstHost = DstY + m.dstPitch * ah;
	m.Height = ah / 2;
	Result = Decoder.CuvidStream ? FCUDAModule::CUDA().cuMemcpy2DAsync(&m, Decoder.CuvidStream) : FCUDAModule::CUDA().cuMemcpy2D(&m);
	check(Result == CUDA_SUCCESS);

	if (Decoder.CuvidStream)
	{
		Result = FCUDAModule::CUDA().cuStreamSynchronize(Decoder.CuvidStream);
		check(Result == CUDA_SUCCESS);
	}


#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (InNewOutput->GPUBuffer.Resource.IsValid())
	{
		InNewOutput->GPUBuffer.Resource->Unmap(0, nullptr);
		InNewOutput->GPUBuffer.Fence->Signal(InNewOutput->GPUBuffer.FenceValue);
	}
#endif
	return true;
}

#endif // ELECTRA_DECODER_HAVE_NVDEC