// Copyright Epic Games, Inc. All Rights Reserved.

#include "NVDECElectraModule.h"
#include "NVDEC/ElectraMediaNVDEC.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"


#if ELECTRA_DECODER_HAVE_NVDEC
	#if defined(ELECTRA_DECODER_NVDEC_USE_D3D12) && ELECTRA_DECODER_NVDEC_USE_D3D12 != 0
		#include "ID3D12DynamicRHI.h"
		#define ENABLE_ELECTRA_NVDEC 1
	#endif
#endif
#ifndef ENABLE_ELECTRA_NVDEC
	#define ENABLE_ELECTRA_NVDEC 0
#endif


#define LOCTEXT_NAMESPACE "NVDECElectraModule"

DEFINE_LOG_CATEGORY(LogNVDECElectraDecoder);

class FNVDECElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
#if ENABLE_ELECTRA_NVDEC
		// If this application won't render anything then we do not need to provide any render hardware acceleration.
		if (!FApp::CanEverRender())
		{
			return;
		}

		// We only support D3D12 at the moment
		switch(GDynamicRHI->GetInterfaceType())
		{
			case ERHIInterfaceType::D3D12:
			{
				FElectraMediaNVDECDecoder::Startup();
				bIsInitialized = true;
				break;
			}
			default:
			{
				UE_LOGF(LogNVDECElectraDecoder, Log, "NVDEC Electra decoder requires D3D12. Plugin will not be used.");
				break;
			}
		}
#endif
	}

	void ShutdownModule() override
	{
#if ENABLE_ELECTRA_NVDEC
		if (bIsInitialized)
		{
			FElectraMediaNVDECDecoder::Shutdown();
			bIsInitialized = false;
		}
#endif
	}

	bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
	bool bIsInitialized = false;
};

IMPLEMENT_MODULE(FNVDECElectraDecoderModule, NVDECElectra);

#undef LOCTEXT_NAMESPACE
